// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryProcessingInterfacesModule.h"
#include "CoreGlobals.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

#include "GeometryProcessingInterfaces/ApproximateActors.h"
#include "GeometryProcessingInterfaces/MeshAutoUV.h"


IMPLEMENT_MODULE(FGeometryProcessingInterfacesModule, GeometryProcessingInterfaces);


void FGeometryProcessingInterfacesModule::StartupModule()
{

}

void FGeometryProcessingInterfacesModule::ShutdownModule()
{
	ApproximateActors = nullptr;
	CombineMeshInstances = nullptr;
	MeshAutoUV = nullptr;
}

namespace
{
	template <typename TModularFeatureInterface>
	TModularFeatureInterface* GetModularFeatureImplementation()
	{
		TArray<TModularFeatureInterface*> AvailableImplementations =
			IModularFeatures::Get().GetModularFeatureImplementations<TModularFeatureInterface>(TModularFeatureInterface::GetModularFeatureName());

		return (AvailableImplementations.Num() > 0) ? AvailableImplementations[0] : nullptr;
	}
}

IGeometryProcessing_ApproximateActors* FGeometryProcessingInterfacesModule::GetApproximateActorsImplementation()
{
	if (ApproximateActors == nullptr)
	{
		ApproximateActors = GetModularFeatureImplementation<IGeometryProcessing_ApproximateActors>();
	}
	return ApproximateActors;
}

IGeometryProcessing_MeshAutoUV* FGeometryProcessingInterfacesModule::GetMeshAutoUVImplementation()
{
	if (MeshAutoUV == nullptr)
	{
		MeshAutoUV = GetModularFeatureImplementation<IGeometryProcessing_MeshAutoUV>();
	}
	return MeshAutoUV;
}


