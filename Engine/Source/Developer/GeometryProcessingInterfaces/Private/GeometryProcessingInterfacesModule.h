// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IGeometryProcessingInterfacesModule.h"


class IGeometryProcessing_ApproximateActors;
class IGeometryProcessing_CombineMeshInstances;
class IGeometryProcessing_MeshAutoUV;


/**
 * Implementation of IGeometryProcessingInterfacesModule (which extends the standard IModuleInterface)
 * to provide access to various "Operation" interfaces
 */
class FGeometryProcessingInterfacesModule : public IGeometryProcessingInterfacesModule
{
public:

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	

	/**
	 * @return implementation of IGeometryProcessing_ApproximateActors, if available, or nullptr (result is cached internally)
	 */
	virtual IGeometryProcessing_ApproximateActors* GetApproximateActorsImplementation() override;

	/**
	 * @return implementation of IGeometryProcessing_MeshAutoUV, if available, or nullptr (result is cached internally)
	 */
	virtual IGeometryProcessing_MeshAutoUV* GetMeshAutoUVImplementation() override;

private:
	IGeometryProcessing_ApproximateActors* ApproximateActors = nullptr;
	IGeometryProcessing_CombineMeshInstances* CombineMeshInstances = nullptr;
	IGeometryProcessing_MeshAutoUV* MeshAutoUV = nullptr;
};
