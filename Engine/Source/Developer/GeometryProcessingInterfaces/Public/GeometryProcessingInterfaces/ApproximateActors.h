// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

class AActor;
class UActorComponent;
class UStaticMesh;
class UMaterial;
class UMaterialInterface;
class UTexture2D;
struct FMeshApproximationSettings;


/**
 * IGeometryProcessing_ApproximateActors is a generic high-level interface to a function/system that
 * outputs a set of new Mesh assets that approximate a set of input Actors.
 * This can be used to provide LOD-generation implementations for systems like ProxyLOD and HLOD.
 * 
 * This is an IModularFeature, and so clients can query/enumerate the available ApproximateActors implementations 
 * based on the ::GetModularFeatureName(). However, the preferred way is to use code like the following:
 * 
 *     IGeometryProcessingInterfacesModule& GeomProcInterfaces = FModuleManager::Get().LoadModuleChecked<IGeometryProcessingInterfacesModule>("GeometryProcessingInterfaces");
 *     IGeometryProcessing_ApproximateActors* ApproxActorsAPI = GeomProcInterfaces.GetApproximateActorsImplementation();
 * 
 * This will automatically determine which available implementation to use (based on any applicable config settings/etc),
 * and cache the result of that decision.
 */
class IGeometryProcessing_ApproximateActors : public IModularFeature
{
public:
	virtual ~IGeometryProcessing_ApproximateActors() = default;

	enum class EMeshDataSourceLODPolicy
	{
		LOD0SourceMeshes,
		LOD0RenderMeshes
	};

	enum class EApproximationPolicy
	{
		MeshAndGeneratedMaterial,
		CollisionMesh
	};

	enum class EGroundPlanePolicy
	{
		NoGroundPlane = 0,
		FixedZHeightGroundPlane = 1
	};

	enum class EGroundPlaneClippingPolicy
	{
		NoClipping = 0,
		DiscardFullyHiddenFaces = 1,
		CutFaces = 2,
		CutFacesAndFill = 3
	};

	enum class EBaseCappingPolicy
	{
		NoBaseCapping = 0,
		ConvexPolygon = 1,
		ConvexSolid = 2
	};

	enum class EOcclusionPolicy : uint8
	{
		None = 0,
		VisibilityBased = 1
	};

	enum class ESimplificationPolicy
	{
		FixedTriangleCount = 0,
		TrianglesPerUnitSqMeter = 1,
		GeometricTolerance = 2
	};

	enum class EUVGenerationPolicy : uint8
	{
		PreferUVAtlas = 0,
		PreferXAtlas = 1,
		PreferPatchBuilder = 2
	};

	enum class ETextureSizePolicy : uint8
	{
		TextureSize = 0,
		TexelDensity = 1,
		CustomTextureSize = 2
	};

	enum class ENaniteFallbackTarget : uint8
	{
		Auto,
		PercentTriangles,
		RelativeError
	};

	/**
	 * Input to approximate, as actors and/or components
	 */
	struct FInput
	{
		TArray<AActor*>				Actors;
		TArray<UActorComponent*>	Components;
	};


	/**
	 * Input options to Actor Approximation process
	 */
	struct FOptions
	{
		// Base path for generated assets. Names will be generated by appending strings to this path.
		FString BasePackagePath;

		// high-level control of the overall approximation process
		EApproximationPolicy BasePolicy = EApproximationPolicy::MeshAndGeneratedMaterial;

		// 
		// Actor/Scene configuration settings
		//

		// control which LOD, and which type of LOD mesh, should be used as the source for the approximation process
		EMeshDataSourceLODPolicy MeshDataLODPolicy = EMeshDataSourceLODPolicy::LOD0SourceMeshes;

		//
		// Mesh Preprocessing settings
		//
		bool bAutoThickenThinParts = false;
		double AutoThickenThicknessMeters = 0.1;

		bool bIgnoreTinyParts = false;
		double TinyPartMaxDimensionMeters = 0.1;

		EBaseCappingPolicy BaseCappingPolicy = EBaseCappingPolicy::NoBaseCapping;
		double BaseThicknessOverrideMeters = 0.0;		// use this thickness for Solid. If zero, use AutoThickenThickness if bAutoThickenThinParts=true, otherwise use WorldSpaceApproximationAccuracyMeters
		double BaseHeightOverrideMeters = 0.0;		// consider this height from MinZ as "base" region. If zero, use 2.0*WorldSpaceApproximationAccuracyMeters

		//
		// Shape Approximation settings
		//

		// Meshing settings (ie for voxelization)
		double WorldSpaceApproximationAccuracyMeters = 1.0;
		int32 ClampVoxelDimension = 1024;

		double WindingThreshold = 0.5;

		bool bApplyMorphology = false;
		double MorphologyDistanceMeters = 0.1;

		EOcclusionPolicy OcclusionPolicy = EOcclusionPolicy::VisibilityBased;
		bool bAddDownwardFacesOccluder = true;

		ESimplificationPolicy MeshSimplificationPolicy = ESimplificationPolicy::FixedTriangleCount;
		int32 FixedTriangleCount = 5000;
		double SimplificationTargetMetric = 0.1;		// interpretation varies depending on MeshSimplificationPolicy
		bool bEnableFastSimplifyPrePass = false;		// enable pre-simplify "fast-collapse" pass, using a relatively aggressive tolerance based on WorldSpaceApproximationAccuracyMeters

		EGroundPlanePolicy GroundPlanePolicy = EGroundPlanePolicy::NoGroundPlane;
		double GroundPlaneZHeight = 0.0;
		
		EGroundPlaneClippingPolicy GroundPlaneClippingPolicy = EGroundPlaneClippingPolicy::NoClipping;

		bool bCalculateHardNormals = true;
		double HardNormalsAngleDeg = 60.0;

		EUVGenerationPolicy UVPolicy = EUVGenerationPolicy::PreferXAtlas;
		double UVAtlasStretchTarget = 0.1;
		int32 PatchBuilderInitialPatchCount = 250;
		double PatchBuilderCurvatureAlignment = 1.0;
		double PatchBuilderMergingThreshold = 1.5;
		double PatchBuilderMaxNormalDeviationDeg = 45.0;

		//
		// Material approximation settings
		//
		int32 RenderCaptureImageSize = 1024;

		// render capture parameters
		double FieldOfViewDegrees = 45.0;
		double NearPlaneDist = 1.0;

		bool bMaximizeBakeParallelism = true;		// if true, photocapture is computed at the same time as mesh generation. Faster but requires more memory.

		//
		// Material output settings
		//

		// A new MIC derived from this material will be created and assigned to the generated mesh
		UMaterialInterface* BakeMaterial = nullptr;		// if null, will use /MeshModelingToolsetExp/Materials/FullMaterialBakeOutputMaterial instead (or the _PackedMRS version if that parameter is enabled)
		FName BaseColorTexParamName = FName("BaseColor");
		FName BaseColorSwitchName = FName("EnableBaseColor");
		bool bBakeBaseColor = true;
		int32 CustomTextureSizeBaseColor = 1024;
		FName RoughnessTexParamName = FName("Roughness");
		FName RoughnessSwitchName = FName("EnableRoughness");
		bool bBakeRoughness = true;
		int32 CustomTextureSizeRoughness = 1024;
		FName MetallicTexParamName = FName("Metallic");
		FName MetallicSwitchName = FName("EnableMetallic");
		bool bBakeMetallic = true;
		int32 CustomTextureSizeMetallic = 1024;
		FName SpecularTexParamName = FName("Specular");
		FName SpecularSwitchName = FName("EnableSpecular");
		bool bBakeSpecular = true;
		int32 CustomTextureSizeSpecular = 1024;
		FName EmissiveTexParamName = FName("Emissive");
		FName EmissiveSwitchName = FName("EnableEmissive");
		bool bBakeEmissive = true;
		int32 CustomTextureSizeEmissive = 1024;
		FName NormalTexParamName = FName("NormalMap");
		FName NormalSwitchName = FName("EnableNormalMap");
		bool bBakeNormalMap = true;
		int32 CustomTextureSizeNormalMap = 1024;
		
		bool bUsePackedMRS = true;
		FName PackedMRSTexParamName = FName("PackedMRS");
		FName PackedMRSSwitchName = FName("EnablePackedMRS");

		// output texture options
		ETextureSizePolicy TextureSizePolicy = ETextureSizePolicy::TextureSize;
		int32 TextureImageSize = 1024;
		float MeshTexelDensity = 0;

		// supersampling parameter
		int32 AntiAliasMultiSampling = 0;


		//
		// Mesh settings
		//

		// Whether to generate a nanite-enabled mesh
		bool bGenerateNaniteEnabledMesh = false;

		/** Which heuristic to use when generating the fallback mesh. */
		ENaniteFallbackTarget NaniteFallbackTarget = ENaniteFallbackTarget::Auto;
	
		/** Percentage of triangles to keep from source mesh for fallback. 1.0 = no reduction, 0.0 = no triangles. */
		float NaniteFallbackPercentTriangles = 1.0f;

		/** Reduce until at least this amount of error is reached relative to size of the mesh */
		float NaniteFallbackRelativeError = 1.0f;

		// Whether ray tracing will be supported on this mesh. Disable this to save memory if the generated mesh will only be rendered in the distance
		bool bSupportRayTracing = true;

		// Whether to allow distance field to be computed for this mesh. Disable this to save memory if the generated mesh will only be rendered in the distance
		bool bAllowDistanceField = true;

		// Whether to generate lightmap uvs for the merged mesh
		bool bGenerateLightmapUVs = false;

		// Whether the UBodySetup on the Asset will be created
		bool bCreatePhysicsBody = true;

		/** Required to optimize mesh in mirrored transform. Double index buffer size. */
		bool bBuildReversedIndexBuffer = true;

		//
		// Debug settings
		//

		// print useful information to the Output Log 
		bool bVerbose = false;

		// create a flattened (ie non-instanced) mesh and save it with _DEBUG suffix. Warning often absolutely enormous!
		bool bWriteDebugMesh = false;

		// todo
		//   - which texture types to generate (BaseColor, Specular, WorldNormal, etc)
		//   - per-texture resolution, and channel/compression settings
		//   - which material to use as input, whether to make new UMaterial or a MIC
		//   - custom Generated-Texture to Material Parameter Name mapping
	};

	enum class EResultCode
	{
		Success,
		MeshGenerationFailed,
		MaterialGenerationFailed,
		UnknownError
	};

	/**
	 * Outputs of an Actor Approximation process
	 */
	struct FResults
	{
		EResultCode ResultCode = EResultCode::UnknownError;

		TArray<UStaticMesh*> NewMeshAssets;

		TArray<UMaterialInterface*> NewMaterials;
		
		TArray<UTexture2D*> NewTextures;

		// todo
	};

	/**
	 * Construct an FOptions from the provided FMeshApproximationSettings.
	 */
	virtual FOptions ConstructOptions(const FMeshApproximationSettings& MeshApproximationSettings)
	{
		check(false);		// not implemented in base class
		return FOptions();
	}

	/**
	 * Top-level driver function that clients call to generate the approximation for a set of input Actors or Components.
	 */
	virtual void ApproximateActors(const FInput& Input, const FOptions& Options, FResults& ResultsOut) 
	{
		check(false);		// not implemented in base class
	}


	// Modular feature name to register for retrieval during runtime
	static const FName GetModularFeatureName()
	{
		return TEXT("GeometryProcessing_ApproximateActors");
	}


	// delegates to pass back info?
	DECLARE_DELEGATE_OneParam(FApproximateActorsCompleteDelegate, FResults&);
	FApproximateActorsCompleteDelegate CompleteDelegate;

};
