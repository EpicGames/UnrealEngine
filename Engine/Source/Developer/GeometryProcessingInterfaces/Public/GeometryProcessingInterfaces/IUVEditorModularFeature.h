// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

/**
 * This interface acts as a connector to the (optional) UV Editor plugin. If the plugin is available,
 * then it can be retrieved via IModularFeatures::Get().GetModularFeatureImplementation or related methods.
 */
class IUVEditorModularFeature : public IModularFeature
{
public:
	virtual ~IUVEditorModularFeature() = default;

	virtual void LaunchUVEditor(const TArray<TObjectPtr<UObject>>& Objects)
	{
		check(false);		// not implemented in base class
	}

	virtual bool CanLaunchUVEditor(const TArray<TObjectPtr<UObject>>& Objects)
	{
		check(false);		// not implemented in base class
		return false;
	}

	// Modular feature name to register for retrieval during runtime
	static const FName GetModularFeatureName()
	{
		return TEXT("UVEditorModularFeature");
	}
};
