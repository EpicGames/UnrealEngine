// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Features/IModularFeature.h"
#include "Math/Vector2D.h"
#include "MeshDescription.h"

/**
 * The CombineMeshInstances modular feature is used to provide a mechanism
 * for merging a set of instances of meshes (ie mesh + transform + materials + ...)
 * into a smaller set of meshes. Generally this involves creating simpler versions
 * of the instances and appending them into one or a small number of combined meshes.
 */
class IGeometryProcessing_MeshAutoUV : public IModularFeature
{
public:
	virtual ~IGeometryProcessing_MeshAutoUV() = default;

	enum class EAutoUVMethod : uint8
	{
		PatchBuilder = 0,
		UVAtlas = 1,
		XAtlas = 2
	};

	struct FOptions
	{
		EAutoUVMethod Method = EAutoUVMethod::PatchBuilder;

		// UVAtlas parameters
		double UVAtlasStretch = 0.11f;
		int UVAtlasNumCharts = 0;

		// XAtlas parameters
		int XAtlasMaxIterations = 1;

		// PatchBuilder parameters
		int NumInitialPatches = 100;
		double CurvatureAlignment = 1.0;
		double MergingThreshold = 1.5;
		double MaxAngleDeviationDeg = 45.0;
		int SmoothingSteps = 5;
		double SmoothingAlpha = 0.25;

		bool bAutoPack = true;
		int PackingTargetWidth = 512;
	};

	enum class EResultCode
	{
		Success,
		UnknownError
	};

	struct FResults
	{
		EResultCode ResultCode = EResultCode::UnknownError;
	};

	virtual FOptions ConstructDefaultOptions() PURE_VIRTUAL(IGeometryProcessing_MeshAutoUV::ConstructDefaultOptions, return FOptions(); );

	virtual void GenerateUVs(FMeshDescription& InOutMesh, const FOptions& Options, FResults& ResultsOut) PURE_VIRTUAL(IGeometryProcessing_MeshAutoUV::GenerateUVs,);

	// Modular feature name to register for retrieval during runtime
	static const FName GetModularFeatureName()
	{
		return TEXT("GeometryProcessing_MeshAutoUV");
	}
};
