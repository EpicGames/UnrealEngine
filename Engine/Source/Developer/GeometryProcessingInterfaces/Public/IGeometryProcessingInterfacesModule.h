// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class IGeometryProcessing_ApproximateActors;
class IGeometryProcessing_CombineMeshInstances;
class IGeometryProcessing_MeshAutoUV;

/**
 * Abstract interface to a Module that provides functions to access
 * different "Operation" interfaces, which are high-level APIs to
 * run complex geometry operations on engine inputs without the caller
 * needing to know about the potential dependencies (ie, the implementations 
 * can be provided by plugins, and this non-plugin module can find them for 
 * the Engine/Editor core, avoiding the no-core-dependencies-on-plugins limitation)
 * 
 * This interface is implemented by FGeometryProcessingInterfacesModule. 
 * Client code that wants access to the operations should get this interface like so:
 * 
 *		IGeometryProcessingInterfacesModule& GeomProcInterfaces = FModuleManager::Get().LoadModuleChecked<IGeometryProcessingInterfacesModule>("GeometryProcessingInterfaces");
 *
 */
class IGeometryProcessingInterfacesModule : public IModuleInterface
{
public:	
	/**
	 * @return implementation of IGeometryProcessing_ApproximateActors, if available
	 */
	virtual IGeometryProcessing_ApproximateActors* GetApproximateActorsImplementation() = 0;

	/**
	 * @return implementation of IGeometryProcessing_MeshAutoUV, if available, or nullptr (result is cached internally)
	 */
	virtual IGeometryProcessing_MeshAutoUV* GetMeshAutoUVImplementation() = 0;

};
